// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationMessages : ModuleRules
	{
		public AutomationMessages(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"AutomationTest"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				});
		}
	}
}
