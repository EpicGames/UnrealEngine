// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutomationState.h"
#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"

#include "AutomationWorkerMessages.generated.h"

USTRUCT()
struct FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Message")
	FGuid InstanceId = FGuid{};
};

/* Worker discovery messages
 *****************************************************************************/

/**
 * Implements a message that is published to find automation workers.
 */
USTRUCT()
struct FAutomationWorkerFindWorkers : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/** Holds the change list number to find workers for. */
	UPROPERTY(EditAnywhere, Category="Message")
	int32 Changelist;

	/** The name of the game. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString GameName;

	/** The name of the process. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString ProcessName;

	/** Holds the session identifier to find workers for. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SessionId;

	/** Default constructor. */
	FAutomationWorkerFindWorkers() : Changelist(0) { }

	/** Creates and initializes a new instance. */
	FAutomationWorkerFindWorkers(int32 InChangelist, const FString& InGameName, const FString& InProcessName, const FGuid& InSessionId)
		: Changelist(InChangelist)
		, GameName(InGameName)
		, ProcessName(InProcessName)
		, SessionId(InSessionId)
	{ }
};


/**
 * Implements a message that is sent in response to FAutomationWorkerFindWorkers.
 */
USTRUCT()
struct FAutomationWorkerFindWorkersResponse : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the device that the worker is running on. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DeviceName;

	/** Holds the name of the worker's application instance. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString InstanceName;

	/** Holds the name of the platform that the worker is running on. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Platform;

	/** Holds the name of the operating system version. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString OSVersionName;

	/** Holds the name of the device model. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString ModelName;

	/** Holds the name of the GPU. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString GPUName;

	/** Holds the name of the CPU model. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString CPUModelName;

	/** Holds the amount of RAM this device has in gigabytes. */
	UPROPERTY(EditAnywhere, Category="Message")
	uint32 RAMInGB;

	/** Holds the name of the current render mode. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString RenderModeName;

	/** Holds the worker's application session identifier. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SessionId;

	/** Holds the name of the current RHI. */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString RHIName;

	/** Default constructor. */
	FAutomationWorkerFindWorkersResponse() : RAMInGB(0) { }
};


/**
 * Implements a message that notifies automation controllers that a worker went off-line.
 */
USTRUCT()
struct FAutomationWorkerWorkerOffline : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()
};


/**
 */
USTRUCT()
struct FAutomationWorkerPing : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()
};


/**
 */
USTRUCT()
struct FAutomationWorkerStartTestSession : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()
};


/**
 */
USTRUCT()
struct FAutomationWorkerStopTestSession : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()
};


/**
*/
USTRUCT()
struct FAutomationWorkerStopTests : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()
};


/**
 */
USTRUCT()
struct FAutomationWorkerPong : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()
};


/**
 * Implements a message for requesting available automation tests from a worker.
 */
USTRUCT()
struct FAutomationWorkerRequestTests : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/** Holds a flag indicating whether the developer directory should be included. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool DeveloperDirectoryIncluded = false;

	/** Holds a flag indicating which tests we'd like to request. */
	UPROPERTY(EditAnywhere, Category="Message")
	uint32 RequestedTestFlags = 0;

	/** Default constructor. */
	FAutomationWorkerRequestTests() = default;

	/** Creates and initializes a new instance. */
	FAutomationWorkerRequestTests(bool InDeveloperDirectoryIncluded, EAutomationTestFlags InRequestedTestFlags)
		: DeveloperDirectoryIncluded(InDeveloperDirectoryIncluded)
		, RequestedTestFlags((uint32)InRequestedTestFlags)
	{
	}
};


/**
 * A single test reply, used by FAutomationWorkerRequestTestsReplyComplete
 */
USTRUCT()
struct FAutomationWorkerSingleTestReply : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString DisplayName;

	UPROPERTY(EditAnywhere, Category="Message")
	FString FullTestPath;

	UPROPERTY(EditAnywhere, Category="Message")
	FString TestName;

	UPROPERTY(EditAnywhere, Category="Message")
	FString TestParameter;

	UPROPERTY(EditAnywhere, Category="Message")
	FString SourceFile;

	UPROPERTY(EditAnywhere, Category="Message")
	int32 SourceFileLine = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	FString AssetPath;

	UPROPERTY(EditAnywhere, Category="Message")
	FString OpenCommand;

	UPROPERTY(EditAnywhere, Category="Message")
	uint32 TestFlags = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	uint32 NumParticipantsRequired = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	FString TestTags;

	/** Default constructor. */
	FAutomationWorkerSingleTestReply() = default;

	/** Creates and initializes a new instance. */
	FAutomationWorkerSingleTestReply(const FAutomationTestInfo& InTestInfo)
	{
		DisplayName = InTestInfo.GetDisplayName();
		FullTestPath = InTestInfo.GetFullTestPath();
		TestName = InTestInfo.GetTestName();
		TestParameter = InTestInfo.GetTestParameter();
		SourceFile = InTestInfo.GetSourceFile();
		SourceFileLine = InTestInfo.GetSourceFileLine();
		AssetPath = InTestInfo.GetAssetPath();
		OpenCommand = InTestInfo.GetOpenCommand();
		TestFlags = (uint32)InTestInfo.GetTestFlags();
		NumParticipantsRequired = InTestInfo.GetNumParticipantsRequired();
		TestTags = InTestInfo.GetTestTags();
	}

	FAutomationTestInfo GetTestInfo() const
	{
		return FAutomationTestInfo(
			DisplayName,
			FullTestPath,
			TestName,
			(EAutomationTestFlags)TestFlags,
			NumParticipantsRequired,
			TestParameter,
			SourceFile,
			SourceFileLine,
			AssetPath,
			OpenCommand,
			TestTags);
	}
};


/**
 * Returns list of all tests
 */
USTRUCT()
struct FAutomationWorkerRequestTestsReplyComplete : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	TArray<FAutomationWorkerSingleTestReply> Tests;
};


/**
 * Implements a message to request the running of automation tests on a worker.
 */
USTRUCT()
struct FAutomationWorkerRunTests : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	uint32 ExecutionCount;

	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	int32 RoleIndex;

	/** Holds the name of the test to run. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString TestName;

	/** Holds the name of the test to run. */
	UPROPERTY()
	FString BeautifiedTestName;

	/** Holds the full test path of the test to run. */
	UPROPERTY()
	FString FullTestPath;

	/** If true, send results to analytics when complete */
	UPROPERTY()
	bool bSendAnalytics;

	/** If true, prune log events from test report on success */
	UPROPERTY()
	bool bPruneLogsOnSuccess;

	/** Default constructor. */
	FAutomationWorkerRunTests( ) :ExecutionCount(0), RoleIndex(0), bSendAnalytics(false), bPruneLogsOnSuccess(false) { }

	/** Creates and initializes a new instance. */
	FAutomationWorkerRunTests( uint32 InExecutionCount, int32 InRoleIndex, FString InTestName, FString InBeautifiedTestName, FString InFullTestPath, bool InSendAnalytics, bool InPruneLogsOnSuccess)
		: ExecutionCount(InExecutionCount)
		, RoleIndex(InRoleIndex)
		, TestName(InTestName)
		, BeautifiedTestName(InBeautifiedTestName)
		, FullTestPath(InFullTestPath)
		, bSendAnalytics(InSendAnalytics)
		, bPruneLogsOnSuccess(InPruneLogsOnSuccess)
	{ }
};


/**
 * Implements a message that is sent in response to FAutomationWorkerRunTests.
 */
USTRUCT()
struct FAutomationWorkerRunTestsReply : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

public:
	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	FString TestName;

	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<FAutomationExecutionEntry> Entries;

	UPROPERTY(EditAnywhere, Category="Message")
	int32 WarningTotal = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	int32 ErrorTotal = 0;

	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	float Duration = 0;

	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	uint32 ExecutionCount = 0;

	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	EAutomationState State = EAutomationState::NotRun;
};


/**
 */
USTRUCT()
struct FAutomationWorkerRequestNextNetworkCommand : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/** */
	UPROPERTY(EditAnywhere, Category="Message")
	uint32 ExecutionCount;

	/** Default constructor. */
	FAutomationWorkerRequestNextNetworkCommand() : ExecutionCount(0) { }

	/** Creates and initializes a new instance. */
	FAutomationWorkerRequestNextNetworkCommand(uint32 InExecutionCount)
		: ExecutionCount(InExecutionCount)
	{ }
};


/**
 */
USTRUCT()
struct FAutomationWorkerNextNetworkCommandReply : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()
};


USTRUCT()
struct FAutomationScreenshotMetadata
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(EditAnywhere, Category="Message")
	FString ScreenShotName;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString VariantName;

	UPROPERTY(EditAnywhere, Category="Message")
	FString Context;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString TestName;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString Notes;

	UPROPERTY(EditAnywhere, Category="Message")
	FGuid Id;
	UPROPERTY(EditAnywhere, Category="Message")
	FString Commit;

	UPROPERTY(EditAnywhere, Category="Message")
	int32 Width;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 Height;

	// RHI Details
	UPROPERTY(EditAnywhere, Category="Message")
	FString Rhi;
	UPROPERTY(EditAnywhere, Category="Message")
	FString Platform;
	UPROPERTY(EditAnywhere, Category="Message")
	FString FeatureLevel;
	UPROPERTY(EditAnywhere, Category="Message")
	bool bIsStereo;

	// Track if ScreenShotMetadata was created with Substrate enabled or not
	UPROPERTY(EditAnywhere, Category="Message")
	bool bIsSubstrate;

	// Hardware Details
	UPROPERTY(EditAnywhere, Category="Message")
	FString Vendor;
	UPROPERTY(EditAnywhere, Category="Message")
	FString AdapterName;
	UPROPERTY(EditAnywhere, Category="Message")
	FString AdapterInternalDriverVersion;
	UPROPERTY(EditAnywhere, Category="Message")
	FString AdapterUserDriverVersion;
	UPROPERTY(EditAnywhere, Category="Message")
	FString UniqueDeviceId;

	// Quality Levels
	UPROPERTY(EditAnywhere, Category="Message")
	float ResolutionQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 ViewDistanceQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 AntiAliasingQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 ShadowQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 GlobalIlluminationQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 ReflectionQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 PostProcessQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 TextureQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 EffectsQuality;
	UPROPERTY(EditAnywhere, Category="Message")
	int32 FoliageQuality;
	UPROPERTY(EditAnywhere, Category = "Message")
	int32 ShadingQuality;

	// Comparison Requests
	UPROPERTY(EditAnywhere, Category="Message")
	bool bHasComparisonRules;
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 ToleranceRed;
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 ToleranceGreen;
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 ToleranceBlue;
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 ToleranceAlpha;
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 ToleranceMinBrightness;
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 ToleranceMaxBrightness;
	UPROPERTY(EditAnywhere, Category="Message")
	float MaximumLocalError;
	UPROPERTY(EditAnywhere, Category="Message")
	float MaximumGlobalError;
	UPROPERTY(EditAnywhere, Category="Message")
	bool bIgnoreAntiAliasing;
	UPROPERTY(EditAnywhere, Category="Message")
	bool bIgnoreColors;

public:

	FAutomationScreenshotMetadata()
		: Width(0)
		, Height(0)
		, bIsStereo(false)
		, bIsSubstrate(false)
		, ResolutionQuality(0.0f)
		, ViewDistanceQuality(0)
		, AntiAliasingQuality(0)
		, ShadowQuality(0)
		, GlobalIlluminationQuality(0)
		, ReflectionQuality(0)
		, PostProcessQuality(0)
		, TextureQuality(0)
		, EffectsQuality(0)
		, FoliageQuality(0)
		, ShadingQuality(0)
		, bHasComparisonRules(0)
		, ToleranceRed(0)
		, ToleranceGreen(0)
		, ToleranceBlue(0)
		, ToleranceAlpha(0)
		, ToleranceMinBrightness(0)
		, ToleranceMaxBrightness(0)
		, MaximumLocalError(0.0f)
		, MaximumGlobalError(0.0f)
		, bIgnoreAntiAliasing(false)
		, bIgnoreColors(false)
	{
	}

	FAutomationScreenshotMetadata(const FAutomationScreenshotData& Data)
	{
		static_assert((sizeof(FAutomationScreenshotMetadata) + sizeof(FString)) == sizeof(FAutomationScreenshotData), "These objects must match in size, to ensure we're copying all the members, except for FAutomationScreenshotData.Path, which we don't copy over.");

		// Human readable name and associated context the screenshot was taken in.
		ScreenShotName = Data.ScreenShotName;
		VariantName = Data.VariantName;
		Context = Data.Context;
		TestName = Data.TestName;
		Notes = Data.Notes;

		// Unique Id so we know if this screenshot has already been imported.
		Id = Data.Id;
		Commit = Data.Commit;

		// Resolution Details
		Width = Data.Width;
		Height = Data.Height;

		// RHI Details
		Rhi = Data.Rhi;
		Platform = Data.Platform;
		FeatureLevel = Data.FeatureLevel;
		bIsStereo = Data.bIsStereo;
		bIsSubstrate = Data.bIsSubstrate;

		// Hardware Details
		Vendor = Data.Vendor;
		AdapterName = Data.AdapterName;
		AdapterInternalDriverVersion = Data.AdapterInternalDriverVersion;
		AdapterUserDriverVersion = Data.AdapterUserDriverVersion;
		UniqueDeviceId = Data.UniqueDeviceId;

		// Quality Levels
		ResolutionQuality = Data.ResolutionQuality;
		ViewDistanceQuality = Data.ViewDistanceQuality;
		AntiAliasingQuality = Data.AntiAliasingQuality;
		ShadowQuality = Data.ShadowQuality;
		GlobalIlluminationQuality = Data.GlobalIlluminationQuality;
		ReflectionQuality = Data.ReflectionQuality;
		PostProcessQuality = Data.PostProcessQuality;
		TextureQuality = Data.TextureQuality;
		EffectsQuality = Data.EffectsQuality;
		FoliageQuality = Data.FoliageQuality;
		ShadingQuality = Data.ShadingQuality;

		// Enabled Features
		// TBD

		// Comparison Requests
		bHasComparisonRules = Data.bHasComparisonRules;
		ToleranceRed = Data.ToleranceRed;
		ToleranceGreen = Data.ToleranceGreen;
		ToleranceBlue = Data.ToleranceBlue;
		ToleranceAlpha = Data.ToleranceAlpha;
		ToleranceMinBrightness = Data.ToleranceMinBrightness;
		ToleranceMaxBrightness = Data.ToleranceMaxBrightness;
		
		MaximumLocalError = Data.MaximumLocalError;
		MaximumGlobalError = Data.MaximumGlobalError;

		bIgnoreAntiAliasing = Data.bIgnoreAntiAliasing;
		bIgnoreColors = Data.bIgnoreColors;
	}

	int32 Compare(const FAutomationScreenshotMetadata& OtherMetadata) const
	{
		int32 Score = 0;

		if ( Width != OtherMetadata.Width || Height != OtherMetadata.Height || bIsStereo != OtherMetadata.bIsStereo )
		{
			return 0;
		}
		else
		{
			Score += 1000;
		}

		if (bIsSubstrate == OtherMetadata.bIsSubstrate)
		{
			Score += 100;
		}

		if (Vendor == OtherMetadata.Vendor)
		{
			Score += 100;
		}

		if (FeatureLevel == OtherMetadata.FeatureLevel)
		{
			Score += 100;
		}

		if (UniqueDeviceId == OtherMetadata.UniqueDeviceId)
		{
			Score += 100;
		}

		if (Rhi == OtherMetadata.Rhi)
		{
			Score += 100;
		}

		if (Platform == OtherMetadata.Platform)
		{
			Score += 10;
		}

		if (AdapterName == OtherMetadata.AdapterName)
		{
			Score += 10;
		}

		if (AdapterInternalDriverVersion == OtherMetadata.AdapterInternalDriverVersion)
		{
			Score += 10;
		}

		if (AdapterUserDriverVersion == OtherMetadata.AdapterUserDriverVersion)
		{
			Score += 10;
		}

		if (ResolutionQuality == OtherMetadata.ResolutionQuality)
		{
			Score += 10;
		}

		if (ViewDistanceQuality == OtherMetadata.ViewDistanceQuality)
		{
			Score += 10;
		}

		if (AntiAliasingQuality == OtherMetadata.AntiAliasingQuality)
		{
			Score += 10;
		}

		if (ShadowQuality == OtherMetadata.ShadowQuality)
		{
			Score += 10;
		}

		if (GlobalIlluminationQuality == OtherMetadata.GlobalIlluminationQuality)
		{
			Score += 10;
		}

		if (ReflectionQuality == OtherMetadata.ReflectionQuality)
		{
			Score += 10;
		}

		if (PostProcessQuality == OtherMetadata.PostProcessQuality)
		{
			Score += 10;
		}

		if (TextureQuality == OtherMetadata.TextureQuality)
		{
			Score += 10;
		}

		if (EffectsQuality == OtherMetadata.EffectsQuality)
		{
			Score += 10;
		}

		if (FoliageQuality == OtherMetadata.FoliageQuality)
		{
			Score += 10;
		}

		return Score;
	}
};

/**
 * Implements a message that is sent in containing a screen shot run during performance test.
 */
USTRUCT()
struct FAutomationWorkerScreenImage : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/** The screen shot data. */
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint8> ScreenImage;

	/** The frame trace data. */
	UPROPERTY(EditAnywhere, Category = "Message")
	TArray<uint8> FrameTrace;

	/** The screen shot name. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString ScreenShotName;

	UPROPERTY(EditAnywhere, Category="Message")
	FAutomationScreenshotMetadata Metadata;
};



/**
 * Implements a message that is sent in containing a screen shot run during performance test.
 */
USTRUCT()
struct FAutomationWorkerImageComparisonResults : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

public:

	FAutomationWorkerImageComparisonResults(FGuid InInstanceId = FGuid{})
		: FAutomationWorkerMessageBase{InInstanceId}
		, bNew(false)
		, bSimilar(false)
		, MaxLocalDifference(0.0)
		, GlobalDifference(0.0)
	{
	}

	FAutomationWorkerImageComparisonResults(
		FGuid InInstanceId,
		FGuid InUniqueId,
		const FString& InScreenshotPath,
		bool InIsNew,
		bool InAreSimilar,
		double InMaxLocalDifference,
		double InGlobalDifference,
		const FString& InErrorMessage,
		const FString& InIncomingFilePath,
		const FString& InReportComparisonFilePath,
		const FString& InReportApprovedFilePath,
		const FString& InReportIncomingFilePath
	)
		: FAutomationWorkerMessageBase{ InInstanceId }
		, UniqueId(InUniqueId)
		, ScreenshotPath(InScreenshotPath)
		, bNew(InIsNew)
		, bSimilar(InAreSimilar)
		, MaxLocalDifference(InMaxLocalDifference)
		, GlobalDifference(InGlobalDifference)
		, ErrorMessage(InErrorMessage)
		, IncomingFilePath(InIncomingFilePath)
		, ReportComparisonFilePath(InReportComparisonFilePath)
		, ReportApprovedFilePath(InReportApprovedFilePath)
		, ReportIncomingFilePath(InReportIncomingFilePath)
	{
	}

	/** The unique id for the comparison. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid UniqueId;

	/** The path of the screenshot. */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString ScreenshotPath;

	/** Was this a new image we've never seen before and have no ground truth for? */
	UPROPERTY(EditAnywhere, Category="Message")
	bool bNew;

	/** Were the images similar?  If they're not you should log an error. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool bSimilar;

	UPROPERTY(EditAnywhere, Category="Message")
	double MaxLocalDifference;

	UPROPERTY(EditAnywhere, Category="Message")
	double GlobalDifference;

	UPROPERTY(EditAnywhere, Category="Message")
	FString ErrorMessage;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString IncomingFilePath;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString ReportComparisonFilePath;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString ReportApprovedFilePath;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString ReportIncomingFilePath;
};


/**
 * Implements a message that handles both storing and requesting ground truth data.
 * for the first time this test is run, it might need to store things, or get things.
 */
USTRUCT()
struct FAutomationWorkerTestDataRequest : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/** The category of the data, this is purely to bucket and separate the ground truth data we store into different directories. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DataType;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DataPlatform;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DataTestName;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DataName;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString JsonData;
};

/**
 * Implements a message that responds to TestDataRequests.
 */
USTRUCT()
struct FAutomationWorkerTestDataResponse : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString JsonData;

	UPROPERTY(EditAnywhere, Category="Message")
	bool bIsNew = false;
};

/**
 * Implements a message to request the performance data for this hardware.
 */
USTRUCT()
struct FAutomationWorkerPerformanceDataRequest : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Platform;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Hardware;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString TestName;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<double> DataPoints;
};

/**
 * Implements a message that responds to PerformanceDataRequest.
 */
USTRUCT()
struct FAutomationWorkerPerformanceDataResponse : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	bool bSuccess = false;

	/**  */
	UPROPERTY(EditAnywhere, Category="Message")
	FString ErrorMessage;
};

/**
 * Implements a message that contains telemetry data point.
 */
USTRUCT()
struct FAutomationWorkerTelemetryItem : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString DataPoint;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	double Measurement;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString Context;

	FAutomationWorkerTelemetryItem() : Measurement(0.0) {}

	FAutomationWorkerTelemetryItem(FString& InDataPoint, double InMeasurement, FString& InContext)
		: DataPoint(InDataPoint)
		, Measurement(InMeasurement)
		, Context(InContext)
	{
	}

	FAutomationWorkerTelemetryItem(const FAutomationTelemetryData& InItem)
		: DataPoint(InItem.DataPoint)
		, Measurement(InItem.Measurement)
		, Context(InItem.Context)
	{
	}
};

/**
 * Implements a message that contains telemetry data.
 */
USTRUCT()
struct FAutomationWorkerTelemetryData : public FAutomationWorkerMessageBase
{
	GENERATED_USTRUCT_BODY()

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString Storage;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString Configuration;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString Platform;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString TestName;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Message")
	TArray<FAutomationWorkerTelemetryItem> Items;
};
